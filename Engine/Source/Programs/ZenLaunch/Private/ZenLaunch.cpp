// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenLaunch.h"

#include "Containers/UnrealString.h"
#include "Experimental/ZenServerInterface.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "String/LexFromString.h"
#include "String/ParseTokens.h"
#include "ProjectUtilities.h"

#include "RequiredProgramMainCPPInclude.h"

DEFINE_LOG_CATEGORY_STATIC(LogZenLaunch, Log, All);

IMPLEMENT_APPLICATION(ZenLaunch, "ZenLaunch");

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FTaskTagScope Scope(ETaskTag::EGameThread);
	ON_SCOPE_EXIT
	{ 
		LLM(FLowLevelMemTracker::Get().UpdateStatsPerFrame());
		RequestEngineExit(TEXT("Exiting"));
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};

	// Allows this program to accept a project argument on the commandline and use project-specific config
	UE::ProjectUtilities::ParseProjectDirFromCommandline(ArgC, ArgV);

	if (int32 Ret = GEngineLoop.PreInit(ArgC, ArgV))
	{
		return Ret;
	}

	constexpr FStringView SponsorNone(TEXTVIEW("None"));
	const TCHAR* CommandLine = FCommandLine::Get();
	TArray<uint32> SponsorProcessIDs;
	bool bHasSponsorNone = false;
	for (FString Token; FParse::Token(CommandLine, Token, /*UseEscape*/ false);)
	{
		TArray<FString> SponsorProcessIDStrings;
		Token.ReplaceInline(TEXT("\""), TEXT(""));
		const auto GetSwitchValues = [Token = FStringView(Token)](FStringView Match, TArray<FString>& OutValues)
		{
			if (Token.StartsWith(Match))
			{
				FStringView Values = Token.RightChop(Match.Len());
				UE::String::ParseTokens(Values, TEXTVIEW(","),
					[&OutValues](FStringView Value)
					{
						OutValues.Emplace(Value);
					},
					UE::String::EParseTokensOptions::Trim | UE::String::EParseTokensOptions::SkipEmpty);
			}
		};

		GetSwitchValues(TEXT("-SponsorProcessID="), SponsorProcessIDStrings);
		GetSwitchValues(TEXT("-Sponsor="), SponsorProcessIDStrings);

		for (const FString& SponsorProcessIDString : SponsorProcessIDStrings)
		{
			if (SponsorNone.Equals(SponsorProcessIDString, ESearchCase::IgnoreCase))
			{
				bHasSponsorNone = true;
				continue;
			}
			uint32 SponsorProcessID = 0;
			LexFromString(SponsorProcessID, SponsorProcessIDString);
			if (SponsorProcessID == 0)
			{
				UE_LOGF(LogZenLaunch, Warning, "Skipping invalid sponsor process ID: %ls", *SponsorProcessIDString);
				continue;
			}

			FProcHandle SponsorProcess = FPlatformProcess::OpenProcess(SponsorProcessID);
			ON_SCOPE_EXIT
			{ 
				FPlatformProcess::CloseProc(SponsorProcess);
			};

			if (!SponsorProcess.IsValid() || !FPlatformProcess::IsProcRunning(SponsorProcess))
			{
				UE_LOGF(LogZenLaunch, Warning, "Skipping sponsor process ID because the process is not accessible and running: %ls", *SponsorProcessIDString);
				continue;
			}

			SponsorProcessIDs.AddUnique(SponsorProcessID);
		}
	}

	if (!bHasSponsorNone && SponsorProcessIDs.IsEmpty())
	{
		UE_LOGF(LogZenLaunch, Error, "No valid sponsor process IDs (or -Sponsor=None) supplied on the commandline. "
			"\n\tPlease supply process IDs via -Sponsor=X,Y,Z. ZenServer will stay running until these processes exit."
			"\n\tOr pass -Sponsor=None to indicate the zenserver should run until closed via a `zen down` call.");
		return 1;
	}
	if (bHasSponsorNone)
	{
		FPlatformMisc::SetEnvironmentVar(TEXT("UE-ZenLimitProcessLifetime"), TEXT("false"));
	}

	UE::Zen::FZenServiceInstance& ZenServiceInstance = UE::Zen::GetDefaultServiceInstance();
	const UE::Zen::FServiceSettings& ZenServiceSettings = ZenServiceInstance.GetServiceSettings();

	if (!ZenServiceSettings.IsAutoLaunch())
	{
		UE_LOGF(LogZenLaunch, Display, "Cannot set ZenServer process lifetime because the ZenServer was manually launched. "
			"\n\tProcess lifetime is settable only for AutoLaunched ZenServers.");
		return 0;
	}

	if (bHasSponsorNone)
	{
		if (ZenServiceInstance.GetServiceSettings().SettingsVariant.Get<UE::Zen::FServiceAutoLaunchSettings>().bLimitProcessLifetime)
		{
			UE_LOGF(LogZenLaunch, Error, "Failed to set zen to run unsponsored; unknown error. "
				"\n\tUE-ZenLimitProcessLifetime=false was set but upon creation bLimitProcessLifetime=true.");
			return 1;
		}

		UE_LOGF(LogZenLaunch, Display, "ZenServer is now running unsponsored; it will remain running until `zen down` is called.");
		return 0;
	}
	else
	{
		if (!ZenServiceInstance.GetServiceSettings().SettingsVariant.Get<UE::Zen::FServiceAutoLaunchSettings>().bLimitProcessLifetime)
		{
			UE_LOGF(LogZenLaunch, Display, "ZenServer settings specify bLimitProcessLifetime=false, so sponsors were ignored. "
				"\n\tZenServer is now running unsponsored; it will remain running until `zen down` is called.");
			return 0;
		}

		if (!ZenServiceInstance.AddSponsorProcessIDs(SponsorProcessIDs))
		{
			UE_LOGF(LogZenLaunch, Error, "Failed to add sponsor process IDs to launched ZenServer.");
			return 1;
		}

		TArray<FString> SponsorProcessIDStrs;
		for (uint32 ID : SponsorProcessIDs)
		{
			SponsorProcessIDStrs.Add(LexToString(ID));
		}
		UE_LOGF(LogZenLaunch, Display, "ZenServer is now running with sponsor ids [%ls]. "
			"\n\tIt will remain running until those processes exit or zen down is called.",
			*FString::Join(SponsorProcessIDStrs, TEXT(",")));
		return 0;
	}
}
