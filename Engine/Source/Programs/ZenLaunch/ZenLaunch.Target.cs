// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class ZenLaunchTarget : TargetRules
{
	public ZenLaunchTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "ZenLaunch";

		// Lean and mean
		bBuildDeveloperTools = false;

		// Building with editor-only data is required for accepting a project argument and loading project-specific configuration
		bBuildWithEditorOnlyData = true;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bCompileAgainstApplicationCore = false;
		bCompileICU = false;

		// to build with automation tests:
		// bForceCompileDevelopmentAutomationTests = true;

		// to enable tracing:
		// bEnableTrace = true;

		// This app is a console application, not a Windows app (sets entry point to main(), instead of WinMain())
		bIsBuildingConsoleApplication = true;
	}
}
